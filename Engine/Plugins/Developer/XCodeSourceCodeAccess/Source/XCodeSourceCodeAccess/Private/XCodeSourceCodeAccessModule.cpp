// Copyright Epic Games, Inc. All Rights Reserved.

#include "XCodeSourceCodeAccessModule.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE( FXCodeSourceCodeAccessModule, XCodeSourceCodeAccess );

void FXCodeSourceCodeAccessModule::StartupModule()
{
	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &XCodeSourceCodeAccessor );
}

void FXCodeSourceCodeAccessModule::ShutdownModule()
{
	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &XCodeSourceCodeAccessor);
}