// Copyright Epic Games, Inc. All Rights Reserved.

#include "XCodeSourceCodeAccessor.h"
#include "DesktopPlatformModule.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "XCodeSourceCodeAccessModule.h"
#include "ISourceCodeAccessModule.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#define LOCTEXT_NAMESPACE "XCodeSourceCodeAccessor"

DEFINE_LOG_CATEGORY_STATIC(LogXcodeAccessor, Log, All);

/** Applescript we use to open XCode */
static const char* OpenXCodeAtFileAndLineAppleScript =
	"on OpenXcodeAtFileAndLine(filepath, linenumber)\n"
	"	set theOffset to offset of \"/\" in filepath\n"
	"	tell application \"{XCODE_PATH}\"\n"
	"		activate\n"
	"		if theOffset is 1 then\n"
	"			open filepath\n"
	"		end if\n"
	"		tell application \"System Events\"\n"
	"			tell process \"Xcode\"\n"
	"				\n"
	"				if theOffset is not 1 then\n"
	"					set bActivated to false\n"
	"					repeat until window \"Open Quickly\" exists\n"
	"						tell application \"{XCODE_PATH}\"\n"
	"							if application \"{XCODE_PATH}\" is not frontmost then\n"
	"								activate\n"
	"							end if\n"
	"						end tell\n"
	"						if application \"{XCODE_PATH}\" is frontmost and bActivated is false then\n"
	"							keystroke \"o\" using {command down, shift down}\n"
	"							set bActivated to true\n"
	"						end if\n"
	"					end repeat\n"
	"					click text field 1 of window \"Open Quickly\"\n"
	"					set value of text field 1 of window \"Open Quickly\" to filepath\n"
	"					keystroke return\n"
	"				end if\n"
	"				\n"
	"				set bActivated to false\n"
	"				repeat until window \"Open Quickly\" exists\n"
	"					tell application \"{XCODE_PATH}\"\n"
	"						if application \"{XCODE_PATH}\" is not frontmost then\n"
	"							activate\n"
	"						end if\n"
	"					end tell\n"
	"					if application \"{XCODE_PATH}\" is frontmost and bActivated is false then\n"
	"						keystroke \"l\" using command down\n"
	"						set bActivated to true\n"
	"					end if\n"
	"				end repeat\n"
	"				\n"
	"				click text field 1 of window \"Open Quickly\"\n"
	"				set value of text field 1 of window \"Open Quickly\" to linenumber\n"
	"				keystroke return\n"
	"				keystroke return\n"
	"			end tell\n"
	"		end tell\n"
	"	end tell\n"
	"end OpenXcodeAtFileAndLine\n"
;

static const char* SaveAllXcodeDocuments =
	"	on SaveAllXcodeDocuments()\n"
	"		tell application \"{XCODE_PATH}\"\n"
	"			save documents\n"
	"		end tell\n"
	"	end SaveAllXcodeDocuments\n"
;

void FXCodeSourceCodeAccessor::Startup()
{
	GetSolutionPath();
}

void FXCodeSourceCodeAccessor::Shutdown()
{
	CachedSolutionPath.Empty();
}

FString FXCodeSourceCodeAccessor::GetSolutionPath() const
{
	if(IsInGameThread())
	{
		if(CachedSolutionPath.Len() <= 0)
		{
			CachedSolutionPath = FPaths::ProjectDir();
			
			if (!FUProjectDictionary::GetDefault().IsForeignProject(CachedSolutionPath))
			{
				CachedSolutionPath = FPaths::Combine(FPaths::RootDir(), + TEXT("UE5 (Mac).xcworkspace/contents.xcworkspacedata"));
			}
			else
			{
				FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(CachedSolutionPath);
				CachedSolutionPath = FPaths::Combine(CachedSolutionPath, BaseName + TEXT(" (Mac).xcworkspace/contents.xcworkspacedata"));
			}
            
            // If modern xcode project doesn't exist, use legacy name instead
            const FString FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead( *CachedSolutionPath );
            if(!FPaths::FileExists( FullPath ))
            {
                CachedSolutionPath.RemoveFromEnd(TEXT(" (Mac).xcworkspace/contents.xcworkspacedata"));
                CachedSolutionPath += TEXT(".xcworkspace/contents.xcworkspacedata");
            }
		}
	}
	return CachedSolutionPath;
}

bool FXCodeSourceCodeAccessor::CanAccessSourceCode() const
{
	return IFileManager::Get().DirectoryExists(*FPlatformMisc::GetXcodePath());
}

FName FXCodeSourceCodeAccessor::GetFName() const
{
	return FName("XCodeSourceCodeAccessor");
}

FText FXCodeSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("XCodeDisplayName", "Xcode");
}

FText FXCodeSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("XCodeDisplayDesc", "Open source code files in XCode");
}

bool FXCodeSourceCodeAccessor::OpenSolution()
{
	return OpenSolutionAtPath(GetSolutionPath());
}

bool FXCodeSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	FString SolutionPath = InSolutionPath;
    FString Extension = FPaths::GetExtension(SolutionPath);
    if (!SolutionPath.EndsWith(TEXT("xcworkspacedata")))
	{
		SolutionPath = SolutionPath + TEXT(".xcworkspace/contents.xcworkspacedata");
	}

    const FString FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead( *SolutionPath );
    UE_LOGF(LogXcodeAccessor, Display, "%ls", *FullPath);
    if ( FPaths::FileExists( FullPath ) )
    {
        FPlatformProcess::LaunchFileInDefaultExternalApplication( *FullPath );
        return true;
	}
    
	return false;
}

bool FXCodeSourceCodeAccessor::DoesSolutionExist() const
{
	FString SolutionPath = GetSolutionPath();
    const FString FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead( *SolutionPath );
    return FPaths::FileExists( FullPath );
}

// ColumnNumber is not supported
bool FXCodeSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
    if (!FPaths::FileExists(FullPath))
    {
        UE_LOGF(LogXcodeAccessor, Warning, "FXCodeSourceCodeAccessor::OpenFileAtLine failed to open non-existent file: %ls", *FullPath);
        return false;
    }
    
    // We use xed to open a file at specified line number through Xcode, it also reuses your currently opened workspace if file exists in project
    int32 ReturnCode = 0;
    FString Errors;
    FString XedCommand = FString::Printf(TEXT("xed -l %d \"%s\""), LineNumber, *FullPath);
    FPlatformProcess::ExecProcess(TEXT("/usr/bin/env"), *XedCommand, &ReturnCode, NULL, &Errors);
    
    if(ReturnCode != 0)
    {
        UE_LOGF(LogXcodeAccessor, Warning, "FXCodeSourceCodeAccessor::OpenFileAtLine failed to open xcode with %ls", *XedCommand);
        // Fallback to trivial implementation when something goes wrong
        FPlatformProcess::LaunchFileInDefaultExternalApplication(*FullPath);
    }
    
    return true;
}

bool FXCodeSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	for ( const FString& SourcePath : AbsoluteSourcePaths )
	{
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*SourcePath);
	}

	return true;
}

bool FXCodeSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	return false;
}

bool FXCodeSourceCodeAccessor::SaveAllOpenDocuments() const
{
	bool ExecutionSucceeded = false;

	FString XcodePath = FPlatformMisc::GetXcodePath();
	XcodePath.RemoveFromEnd(TEXT("/Contents/Developer"));
	
	NSAppleScript* AppleScript = nil;
	
	NSString* AppleScriptString = [NSString stringWithCString:SaveAllXcodeDocuments encoding:NSUTF8StringEncoding];
	AppleScriptString = [AppleScriptString stringByReplacingOccurrencesOfString:@"{XCODE_PATH}" withString:XcodePath.GetNSString()];
	AppleScript = [[NSAppleScript alloc] initWithSource:AppleScriptString];
	
	int PID = [[NSProcessInfo processInfo] processIdentifier];
	NSAppleEventDescriptor* ThisApplication = [NSAppleEventDescriptor descriptorWithDescriptorType:typeKernelProcessID bytes:&PID length:sizeof(PID)];
	
	NSAppleEventDescriptor* ContainerEvent = [NSAppleEventDescriptor appleEventWithEventClass:'ascr' eventID:'psbr' targetDescriptor:ThisApplication returnID:kAutoGenerateReturnID transactionID:kAnyTransactionID];
	
	[ContainerEvent setParamDescriptor:[NSAppleEventDescriptor descriptorWithString:@"SaveAllXcodeDocuments"] forKeyword:'snam'];
	
	NSDictionary* ExecutionError = nil;
	[AppleScript executeAppleEvent:ContainerEvent error:&ExecutionError];
	if(ExecutionError == nil)
	{
		ExecutionSucceeded = true;
	}
	else
	{
		UE_LOGF(LogXcodeAccessor, Error, "%ls", *FString([ExecutionError description]));
	}
	
	[AppleScript release];
	return ExecutionSucceeded;
}

void FXCodeSourceCodeAccessor::Tick(const float DeltaTime)
{
}

#undef LOCTEXT_NAMESPACE
