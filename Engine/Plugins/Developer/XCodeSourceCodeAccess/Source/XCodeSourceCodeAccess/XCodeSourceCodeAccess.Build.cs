// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class XCodeSourceCodeAccess : ModuleRules
	{
        public XCodeSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SourceCodeAccess",
					"DesktopPlatform"
				}
			);

			ShortName = "XCodeSCA";
		}
	}
}
