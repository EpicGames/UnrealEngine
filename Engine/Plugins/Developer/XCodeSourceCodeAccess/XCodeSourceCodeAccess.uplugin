{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "XCode Integration",
	"Description" : "Allows access to source code in XCode.",
	"Category" : "Programming",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": [ "UnrealFrontend", "UnrealInsights" ],
	"SupportedTargetPlatforms":  [ "Mac" ],
	"Modules" :
	[
		{
			"Name" : "XCodeSourceCodeAccess",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Mac"
			],
			"ProgramAllowList" : [ "UnrealFrontend", "UnrealInsights" ]
		}
	]
}